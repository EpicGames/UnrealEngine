{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Concert Replication Scripting",
	"Description": "Exposes Concert Replication types for scripting, e.g. in Blueprints",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
        {
          "Name": "ConcertReplicationScripting",
          "Type": "Runtime",
          "LoadingPhase": "Default"
        },
		{
          "Name": "ConcertReplicationScriptingEditor",
          "Type": "Editor",
          "LoadingPhase": "Default"
        }
	],
	"Plugins": [
		{
			"Name": "ConcertSharedSlate",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		}
	]
}