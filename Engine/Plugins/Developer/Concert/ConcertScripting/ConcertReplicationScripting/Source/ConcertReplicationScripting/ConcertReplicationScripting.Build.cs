// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertReplicationScripting : ModuleRules
	{
		public ConcertReplicationScripting(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"Core",
					"CoreUObject",
					"Engine",
					
					// Concert
					"ConcertSyncCore"
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// Concert
					"ConcertTransport", // For LogConcert
				}
			);

			// Hacky but we decided to keep it like this to avoid exceeding the 200 character path limit
			ShortName = "CS";
		}
	}
}