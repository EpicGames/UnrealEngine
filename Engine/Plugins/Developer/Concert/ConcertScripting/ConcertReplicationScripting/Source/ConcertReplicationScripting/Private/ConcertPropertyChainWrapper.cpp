// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertPropertyChainWrapper.h"

uint32 GetTypeHash(const FConcertPropertyChainWrapper& ChainWrapper)
{
	return GetTypeHash(ChainWrapper.PropertyChain);
}