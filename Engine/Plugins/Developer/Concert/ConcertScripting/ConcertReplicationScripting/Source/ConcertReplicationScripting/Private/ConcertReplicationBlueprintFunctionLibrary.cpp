// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertReplicationBlueprintFunctionLibrary.h"

#include "ConcertLogGlobal.h"
#include "ConcertPropertyChainWrapper.h"
#include "Replication/PropertyChainUtils.h"

bool UConcertReplicationBlueprintFunctionLibrary::MakePropertyChainByLiteralPath(const TSubclassOf<UObject>& Class, const TArray<FName>& PathToProperty, FConcertPropertyChainWrapper& Result)
{
	if (!Class)
	{
		UE_LOGF(LogConcert, Error, "UConcertReplicationBlueprintFunctionLibrary::MakePropertyChainByLiteralPath: Invalid class!")
		return false;
	}

	TOptional<FConcertPropertyChain> PropertyChain = FConcertPropertyChain::CreateFromPath(**Class, PathToProperty);
	if (PropertyChain)
	{
		constexpr bool bLogOnFail = false;
		const FProperty* Property = PropertyChain->ResolveProperty(**Class, bLogOnFail);
		if (!ensure(Property) || !UE::ConcertSyncCore::PropertyChain::IsReplicatableProperty(*Property))
		{
			UE_LOGF(LogConcert, Error, "UConcertReplicationBlueprintFunctionLibrary::MakePropertyChainByLiteralPath: Property path [%ls] resolved to a property that is not replicatable!",
				*FString::JoinBy(PathToProperty, TEXT(", "), [](const FName& Name){ return TEXT("\"") + Name.ToString() + TEXT("\""); })
				);
			return false;
		}

		Result.PropertyChain = MoveTemp(*PropertyChain);
		return true;
	}

	UE_LOGF(LogConcert, Error, "UConcertReplicationBlueprintFunctionLibrary::MakePropertyChainByLiteralPath: Path [%ls] does not point to any property in class %ls or it is not replicatable!",
		*FString::JoinBy(PathToProperty, TEXT(", "), [](const FName& Name){ return TEXT("\"") + Name.ToString() + TEXT("\""); }),
		*Class->GetPathName()
		);
	return false;
}

TArray<FConcertPropertyChainWrapper> UConcertReplicationBlueprintFunctionLibrary::GetPropertiesIn(const TSubclassOf<UObject>& Class, FPropertyChainPredicate Filter)
{
	if (!Class)
	{
		UE_LOGF(LogConcert, Error, "UConcertReplicationBlueprintFunctionLibrary::GetPropertiesIn: No class set!")
		return {};
	}

	TArray<FConcertPropertyChainWrapper> Result;  
	UE::ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(
		**Class,
		[&Filter, &Result](FConcertPropertyChain&& PropertyChain)
		{
			FConcertPropertyChainWrapper Wrapper { MoveTemp(PropertyChain) };
			if (!Filter.IsBound() || Filter.Execute(Wrapper))
			{
				Result.Emplace(MoveTemp(Wrapper.PropertyChain));
			}
			return EBreakBehavior::Continue;
		});
	return Result;
}

TArray<FConcertPropertyChainWrapper> UConcertReplicationBlueprintFunctionLibrary::GetAllProperties(const TSubclassOf<UObject>& Class)
{
	if (!Class)
	{
		UE_LOGF(LogConcert, Error, "UConcertReplicationBlueprintFunctionLibrary::GetPropertiesIn: No class set!")
		return {};
	}

	TArray<FConcertPropertyChainWrapper> Result;  
	UE::ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(
		**Class,
		[&Result](FConcertPropertyChain&& PropertyChain)
		{
			Result.Emplace(MoveTemp(PropertyChain));
			return EBreakBehavior::Continue;
		});
	return Result;
}

TArray<FConcertPropertyChainWrapper> UConcertReplicationBlueprintFunctionLibrary::GetChildProperties(const FConcertPropertyChainWrapper& Parent, const TSubclassOf<UObject>& Class, bool bOnlyDirect)
{
	if (!Class)
	{
		UE_LOGF(LogConcert, Error, "UConcertReplicationBlueprintFunctionLibrary::GetPropertiesIn: No class set!")
		return {};
	}

	TArray<FConcertPropertyChainWrapper> Result;  
	UE::ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(
		**Class,
		[bOnlyDirect, &Result, &Parent](FConcertPropertyChain&& PropertyChain)
		{
			if ((bOnlyDirect && PropertyChain.IsDirectChildOf(Parent.PropertyChain)) || (!bOnlyDirect && PropertyChain.IsChildOf(Parent.PropertyChain)))
			{
				Result.Emplace(MoveTemp(PropertyChain));
			}
			return EBreakBehavior::Continue;
		});
	return Result;
}

FString UConcertReplicationBlueprintFunctionLibrary::ToString(const FConcertPropertyChainWrapper& PropertyChain)
{
	return PropertyChain.PropertyChain.ToString();
}

const TArray<FName>& UConcertReplicationBlueprintFunctionLibrary::GetPropertyStringPath(const FConcertPropertyChainWrapper& Path)
{
	return Path.PropertyChain.GetPathToProperty();
}

FName UConcertReplicationBlueprintFunctionLibrary::GetPropertyFromRoot(const FConcertPropertyChainWrapper& Path, int32 Index)
{
	const TArray<FName>& StringPath = Path.PropertyChain.GetPathToProperty();
	return StringPath.IsValidIndex(Index) ? StringPath[Index] : NAME_None;
}

FName UConcertReplicationBlueprintFunctionLibrary::GetPropertyFromLeaf(const FConcertPropertyChainWrapper& Path, int32 Index)
{
	const TArray<FName>& StringPath = Path.PropertyChain.GetPathToProperty();
	const int32 RealIndex = StringPath.Num() - 1 - Index;
	return StringPath.IsValidIndex(RealIndex) ? StringPath[RealIndex] : NAME_None;
}

bool UConcertReplicationBlueprintFunctionLibrary::IsChildOf(const FConcertPropertyChainWrapper& ToTest, const FConcertPropertyChainWrapper& Parent)
{
	return ToTest.PropertyChain.IsChildOf(Parent.PropertyChain);
}

bool UConcertReplicationBlueprintFunctionLibrary::IsDirectChildOf(const FConcertPropertyChainWrapper& ToTest, const FConcertPropertyChainWrapper& Parent)
{
	return ToTest.PropertyChain.IsDirectChildOf(Parent.PropertyChain);
}
