// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertReplicationScriptingModule.h"

#include "Modules/ModuleManager.h"

namespace UE::ConcertReplicationScripting
{
	void FConcertReplicationScriptingModule::StartupModule()
	{}

	void FConcertReplicationScriptingModule::ShutdownModule()
	{}
}

IMPLEMENT_MODULE(UE::ConcertReplicationScripting::FConcertReplicationScriptingModule, ConcertReplicationScripting);