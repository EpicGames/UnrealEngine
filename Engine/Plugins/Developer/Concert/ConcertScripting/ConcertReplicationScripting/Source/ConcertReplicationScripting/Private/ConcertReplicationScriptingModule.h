// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::ConcertReplicationScripting
{
	class FConcertReplicationScriptingModule : public IModuleInterface
	{
	public:
	
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface
	};
}