// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"
#include "ConcertPropertyChainWrapper.generated.h"

USTRUCT(BlueprintType, meta = (DisplayName = "Concert Property Chain"))
struct CONCERTREPLICATIONSCRIPTING_API FConcertPropertyChainWrapper
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertPropertyChain PropertyChain;
	
	FConcertPropertyChainWrapper() = default;
	explicit FConcertPropertyChainWrapper(FConcertPropertyChain InPropertyChain)
		: PropertyChain(MoveTemp(InPropertyChain))
	{}

	friend bool operator==(const FConcertPropertyChainWrapper& Left, const FConcertPropertyChainWrapper& Right) { return Left.PropertyChain == Right.PropertyChain; }
	friend bool operator!=(const FConcertPropertyChainWrapper& Left, const FConcertPropertyChainWrapper& Right) { return !(Left == Right); }
};

CONCERTREPLICATIONSCRIPTING_API uint32 GetTypeHash(const FConcertPropertyChainWrapper& ChainWrapper);