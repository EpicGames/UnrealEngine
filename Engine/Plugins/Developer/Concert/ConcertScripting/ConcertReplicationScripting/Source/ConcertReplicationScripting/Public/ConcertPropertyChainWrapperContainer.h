// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertPropertyChainWrapper.h"
#include "Containers/Array.h"
#include "ConcertPropertyChainWrapperContainer.generated.h"

/** Special array of property chains. Used to allow for detail customization. */
USTRUCT(BlueprintType, meta = (DisplayName = "Concert Property Chain Container"))
struct CONCERTREPLICATIONSCRIPTING_API FConcertPropertyChainWrapperContainer
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Concert|Replication")
	TArray<FConcertPropertyChainWrapper> PropertyChains;
};