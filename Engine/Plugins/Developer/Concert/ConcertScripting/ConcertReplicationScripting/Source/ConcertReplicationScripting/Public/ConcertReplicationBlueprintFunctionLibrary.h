// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"
#include "ConcertReplicationBlueprintFunctionLibrary.generated.h"

struct FConcertPropertyChainWrapper;

UCLASS(BlueprintType)
class CONCERTREPLICATIONSCRIPTING_API UConcertReplicationBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FPropertyChainPredicate, const FConcertPropertyChainWrapper&, PathToProperty);

	/******************** Factories ********************/
	
	/**
	 * Tries to make concert property chain by specifying a path manually.
	 * 
	 * @param Class The class in which to search for the property path 
	 * @param PathToProperty The path to the property
	 * @param Result The resulting path
	 * 
	 * @return Whether Result is valid.
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static bool MakePropertyChainByLiteralPath(const TSubclassOf<UObject>& Class, const TArray<FName>& PathToProperty, FConcertPropertyChainWrapper& Result);
	
	/**
	 * Builds an array of property paths that pass the given filter.
	 * 
	 * @param Class The class in which to search for properties
	 * @param Filter Decides whether a property path should be included
	 * 
	 * @return An array of all property paths in the given class that passed the filter
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication", meta = (KeyWords = "Get Properties In Find Enumerate List"))
	static TArray<FConcertPropertyChainWrapper> GetPropertiesIn(const TSubclassOf<UObject>& Class, FPropertyChainPredicate Filter);

	/**
	 * Gets all properties in the class that are valid for replicating.
	 * @param Class The class in which to search
	 * @return All properties in the class that are valid for replicating.
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static TArray<FConcertPropertyChainWrapper> GetAllProperties(const TSubclassOf<UObject>& Class);
	
	/**
	 * Returns all child properties of Parent that are valid for replicating.
	 * @param Parent The property of which to find child properties
	 * @param Class The class in which to search
	 * @param bOnlyDirect Whether you only want direct children of Parent
	 * @return All child properties of Parent that are valid for replicating.
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication", meta = (ScriptMethod))
	static TArray<FConcertPropertyChainWrapper> GetChildProperties(const FConcertPropertyChainWrapper& Parent, const TSubclassOf<UObject>& Class, bool bOnlyDirect = false);

	/** Converts the the property to a string. */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication", meta = (ScriptMethod))
	static FString ToString(const FConcertPropertyChainWrapper& PropertyChain);
	
	/******************** Getters ********************/
	
	/**
	 * Gets the path as string array.
	 * Example: ["RelativeLocation", "X"]
	 *
	 * @param Path The property path to extract from
	 *
	 * @return The path as string array
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static const TArray<FName>& GetPropertyStringPath(const FConcertPropertyChainWrapper& Path);

	/**
	 * Gets the property at Index starting from the root most property.
	 * Example: The root Index 0 for ["RelativeLocation", "X"] would return "RelativeLocation".
	 * 
	 * @param Path The property path to get the sub-property from.
	 * @param Index The index in the path counting from the left, root property.
	 * 
	 * @return The property name at Index or None if Index is invalid.
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static FName GetPropertyFromRoot(const FConcertPropertyChainWrapper& Path, int32 Index = 0);

	/**
	 * Gets the property at Index starting from the leaf most property.
	 * Example: The leaf Index 0 for ["RelativeLocation", "X"] would return "X".
	 * 
	 * @param Path The property path to get the sub-property from.
	 * @param Index The index in the path counting from the right, leaf property.
	 * 
	 * @return The property name at Index or None if Index is invalid.
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static FName GetPropertyFromLeaf(const FConcertPropertyChainWrapper& Path, int32 Index = 0);

	/**
	 * Checks whether ToTest is a child property of Parent.
	 * 
	 * @param ToTest The property for which to check whether it is a child property
	 * @param Parent The property which is supposed to be the parent of ToTest
	 * 
	 * @return Whether ToTest is a child property of Parent
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static bool IsChildOf(const FConcertPropertyChainWrapper& ToTest, const FConcertPropertyChainWrapper& Parent);

	/**
	 * Checks whether ToTest is a direct child property of Parent.
	 * 
	 * @param ToTest The property for which to check whether it is a child property
	 * @param Parent The property which is supposed to be the parent of ToTest
	 * 
	 * @return Whether ToTest is a direct child property of Parent
	 */
	UFUNCTION(BlueprintPure, Category = "Concert|Replication")
	static bool IsDirectChildOf(const FConcertPropertyChainWrapper& ToTest, const FConcertPropertyChainWrapper& Parent);
};