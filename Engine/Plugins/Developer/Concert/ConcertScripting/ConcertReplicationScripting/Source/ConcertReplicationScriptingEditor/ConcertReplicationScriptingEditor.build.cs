// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertReplicationScriptingEditor : ModuleRules
	{
		public ConcertReplicationScriptingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					
					// Concert
					"ConcertReplicationScripting",
					"ConcertSyncCore"
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"InputCore",
					"Slate",
					"SlateCore",
					"PropertyEditor",
					"UnrealEd", 
					
					// Concert
					"ConcertTransport", // For LogConcert
					"ConcertSharedSlate"
				}
			);
            
			// Hacky but we decided to keep it like this to avoid exceeding the 200 character path limit
			ShortName = "CSE";
		}
	}
}
