// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertReplicationScriptingEditorModule.h"

#include "ConcertPropertyChainWrapper.h"
#include "ConcertPropertyChainWrapperContainer.h"
#include "Customization/ConcertPropertyContainerCustomization.h"
#include "Customization/ConcertPropertyCustomization.h"
#include "ReplicationScriptingStyle.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

namespace UE::ConcertReplicationScriptingEditor
{
	void FConcertReplicationScriptingEditorModule::StartupModule()
	{
		FReplicationScriptingStyle::Initialize();
		
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(
			FConcertPropertyChainWrapper::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConcertPropertyCustomization::MakeInstance, &SharedClassRememberer)
			);
		PropertyModule.RegisterCustomPropertyTypeLayout(
			FConcertPropertyChainWrapperContainer::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConcertPropertyContainerCustomization::MakeInstance, &SharedClassRememberer)
			);
	}

	void FConcertReplicationScriptingEditorModule::ShutdownModule()
	{
		FReplicationScriptingStyle::Shutdown();
	}
}

IMPLEMENT_MODULE(UE::ConcertReplicationScriptingEditor::FConcertReplicationScriptingEditorModule, ConcertReplicationScriptingEditor);