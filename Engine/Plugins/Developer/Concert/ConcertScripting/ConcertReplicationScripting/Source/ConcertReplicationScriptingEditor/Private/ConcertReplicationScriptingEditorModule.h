// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Util/ClassRememberer.h"

namespace UE::ConcertReplicationScriptingEditor
{
	class FConcertReplicationScriptingEditorModule : public IModuleInterface
	{
	public:
	
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

	private:

		/** Customizations use this cache so users do not constantly have to re-select the class in the drop-down menus. */
		FClassRememberer SharedClassRememberer;
	};
}
