// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertPropertyContainerCustomization.h"

#include "ConcertPropertyChainWrapper.h"
#include "ConcertPropertyChainWrapperContainer.h"
#include "Util/ClassRememberer.h"
#include "Widgets/SConcertPropertyChainCombo.h"

#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FConcertPropertyContainerCustomization"

namespace UE::ConcertReplicationScriptingEditor
{
	TSharedRef<IPropertyTypeCustomization> FConcertPropertyContainerCustomization::MakeInstance(FClassRememberer* ClassRememberer)
	{
		check(ClassRememberer);
		return MakeShared<FConcertPropertyContainerCustomization>(*ClassRememberer);
	}

	FConcertPropertyContainerCustomization::~FConcertPropertyContainerCustomization()
	{
		FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	}

	void FConcertPropertyContainerCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		PropertyHandle = StructPropertyHandle;
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FConcertPropertyContainerCustomization::RefreshPropertiesAndUpdateUI));
		PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FConcertPropertyContainerCustomization::RefreshPropertiesAndUpdateUI));
		FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &FConcertPropertyContainerCustomization::OnObjectTransacted);
		RefreshProperties();
		
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(FMargin(0,2,0,1))
			[
				SAssignNew(PropertyComboBox, SConcertPropertyChainCombo)
				.InitialClassSelection(ClassRememberer.GetLastUsedContextClassFor(*PropertyHandle))
				.IsEditable(!PropertyHandle->IsEditConst())
				.ContainedProperties(&Properties)
				.OnClassChanged_Lambda([this](const UClass* Class)
				{
					ClassRememberer.OnUseClass(*PropertyHandle, Class);
				})
				.OnPropertySelectionChanged(this, &FConcertPropertyContainerCustomization::OnPropertySelectionChanged)
			]
		];
	}

	void FConcertPropertyContainerCustomization::RefreshProperties()
	{
		TArray<const void*> RawStructData;
		PropertyHandle->AccessRawData(RawStructData);

		Properties.Reset();
		bHasMultipleValues = false;

		// Sets are easier to compare than arrays
		using FChainSet = TSet<FConcertPropertyChainWrapper, DefaultKeyFuncs<FConcertPropertyChainWrapper>, TInlineSetAllocator<64>>;
		TOptional<FChainSet> SharedValue;
		
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			if (!RawStructData[Idx])
			{
				continue;
			}

			const FConcertPropertyChainWrapperContainer& CurrentValue = *(FConcertPropertyChainWrapperContainer*)RawStructData[Idx];
			Algo::Transform(CurrentValue.PropertyChains, Properties, [](const FConcertPropertyChainWrapper& Wrapper){ return Wrapper.PropertyChain; });

			FChainSet ValueAsSet(CurrentValue.PropertyChains);
			if (!SharedValue)
			{
				SharedValue.Emplace(ValueAsSet);
			}

			const bool bValueIsDifferent = SharedValue->Num() != ValueAsSet.Num() || !SharedValue->Includes(ValueAsSet);
			bHasMultipleValues |= bValueIsDifferent;
		}
	}

	void FConcertPropertyContainerCustomization::RefreshPropertiesAndUpdateUI()
	{
		RefreshProperties();
		PropertyComboBox->RefreshPropertyContent();
	}

	void FConcertPropertyContainerCustomization::OnPropertySelectionChanged(const FConcertPropertyChain& Property, bool bIsSelected)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeProperty", "Modify property container"));
		PropertyHandle->NotifyPreChange();
		ON_SCOPE_EXIT{ PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet); };
		
		if (bIsSelected)
		{
			Properties.Add(Property);
		}
		else
		{
			Properties.Remove(Property);
		}
		
		// And this updates the underlying property data
		TArray<void*> RawStructData;
		PropertyHandle->AccessRawData(RawStructData);
		FConcertPropertyChainWrapperContainer NewValue;
		Algo::Transform(Properties, NewValue.PropertyChains, [](const FConcertPropertyChain& PropertyChain){ return FConcertPropertyChainWrapper{ PropertyChain }; });
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			FConcertPropertyChainWrapperContainer& Value = *(FConcertPropertyChainWrapperContainer*)RawStructData[Idx];
			Value = NewValue;
		}
		
		// This updates the combo button's content. 
		PropertyComboBox->RefreshPropertyContent();
	}

	void FConcertPropertyContainerCustomization::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent)
	{
		TArray<UObject*> CustomizedObjects;
		PropertyHandle->GetOuterObjects(CustomizedObjects);
		if (CustomizedObjects.Contains(Object))
		{
			RefreshPropertiesAndUpdateUI();
		}
	}
}

#undef LOCTEXT_NAMESPACE