// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Widgets/SConcertPropertyChainCombo.h"

struct FConcertPropertyChain;
class IDetailChildrenBuilder;

namespace UE::ConcertReplicationScriptingEditor
{
	class FClassRememberer;

	/** Shows a combo button that allows selecting a property from a class. */
	class FConcertPropertyContainerCustomization : public IPropertyTypeCustomization
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
	public:
	
		static TSharedRef<IPropertyTypeCustomization> MakeInstance(FClassRememberer* ClassRememberer);
		
		virtual ~FConcertPropertyContainerCustomization() override;

		//~ Begin IPropertyTypeCustomization Interface
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
		//~ End IPropertyTypeCustomization Interface

	private:
		
		FConcertPropertyContainerCustomization(FClassRememberer& InClassRememberer)
			: ClassRememberer(InClassRememberer)
		{}

		/** Customizations use this cache so users do not constantly have to re-select the class in the drop-down menus. */
		FClassRememberer& ClassRememberer;
		
		/** Handle to the property containing the FConcertPropertyChainWrapper struct. */
		TSharedPtr<IPropertyHandle> PropertyHandle;

		/** The UI displayed for the property. */
		TSharedPtr<SConcertPropertyChainCombo> PropertyComboBox;

		/**
		 * Dummy passed in to the SConcertPropertyChainCombo.
		 * Holds all the values from the different property sources.
		 */
		TSet<FConcertPropertyChain> Properties;
		bool bHasMultipleValues = false;

		void RefreshProperties();
		void RefreshPropertiesAndUpdateUI();
		
		void OnPropertySelectionChanged(const FConcertPropertyChain& Property, bool bIsSelected);
		void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);
	};
}


