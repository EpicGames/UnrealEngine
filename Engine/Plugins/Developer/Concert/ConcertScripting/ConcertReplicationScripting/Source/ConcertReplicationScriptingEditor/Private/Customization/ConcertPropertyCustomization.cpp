// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertPropertyCustomization.h"

#include "ConcertPropertyChainWrapper.h"
#include "Util/ClassRememberer.h"
#include "Widgets/SConcertPropertyChainCombo.h"

#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FConcertPropertyCustomization"

namespace UE::ConcertReplicationScriptingEditor
{
	TSharedRef<IPropertyTypeCustomization> FConcertPropertyCustomization::MakeInstance(FClassRememberer* ClassRememberer)
	{
		check(ClassRememberer);
		return MakeShared<FConcertPropertyCustomization>(*ClassRememberer);
	}

	FConcertPropertyCustomization::~FConcertPropertyCustomization()
	{
		FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	}

	void FConcertPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		PropertyHandle = StructPropertyHandle;
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FConcertPropertyCustomization::RefreshPropertiesAndUpdateUI));
		FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &FConcertPropertyCustomization::OnObjectTransacted);
		RefreshProperties();
		
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(FMargin(0,2,0,1))
			[
				SAssignNew(PropertyComboBox, SConcertPropertyChainCombo)
				.InitialClassSelection(ClassRememberer.GetLastUsedContextClassFor(*PropertyHandle))
				.IsEditable(!PropertyHandle->IsEditConst())
				.ContainedProperties(&Properties)
				.OnClassChanged_Lambda([this](const UClass* Class)
				{
					ClassRememberer.OnUseClass(*PropertyHandle, Class);
				})
				.OnPropertySelectionChanged(this, &FConcertPropertyCustomization::OnPropertySelectionChanged)
			]
		];
	}

	void FConcertPropertyCustomization::RefreshProperties()
	{
		TArray<const void*> RawStructData;
		PropertyHandle->AccessRawData(RawStructData);

		Properties.Reset();
		bHasMultipleValues = false;

		TOptional<FConcertPropertyChain> SharedValue;
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			if (!RawStructData[Idx])
			{
				continue;
			}

			const FConcertPropertyChainWrapper& CurrentValue = *(FConcertPropertyChainWrapper*)RawStructData[Idx];
			Properties.Add(CurrentValue.PropertyChain);
			
			if (!SharedValue)
			{
				SharedValue = CurrentValue.PropertyChain;
			}

			const bool bValueIsDifferent = *SharedValue != CurrentValue.PropertyChain;
			bHasMultipleValues |= bValueIsDifferent;
		}
	}

	void FConcertPropertyCustomization::RefreshPropertiesAndUpdateUI()
	{
		RefreshProperties();
		PropertyComboBox->RefreshPropertyContent();
	}

	void FConcertPropertyCustomization::OnPropertySelectionChanged(const FConcertPropertyChain& Property, bool bIsSelected)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeProperty", "Modify property"));
		PropertyHandle->NotifyPreChange();
		ON_SCOPE_EXIT{ PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet); };
		
		// And this updates the underlying property data
		TArray<void*> RawStructData;
		PropertyHandle->AccessRawData(RawStructData);
		FConcertPropertyChainWrapper NewValue = bIsSelected ? FConcertPropertyChainWrapper{ Property } : FConcertPropertyChainWrapper{};
		for (int32 Idx = 0; Idx < RawStructData.Num(); ++Idx)
		{
			FConcertPropertyChainWrapper& Value = *(FConcertPropertyChainWrapper*)RawStructData[Idx];
			Value = NewValue;
		}
		
		// This makes sure the checkboxes display the correct state ...
		Properties = { NewValue.PropertyChain };
		// ... and this updates the combo button's content. 
		PropertyComboBox->RefreshPropertyContent();
	}

	void FConcertPropertyCustomization::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent)
	{
		TArray<UObject*> CustomizedObjects;
		PropertyHandle->GetOuterObjects(CustomizedObjects);
		if (CustomizedObjects.Contains(Object))
		{
			RefreshPropertiesAndUpdateUI();
		}
	}
}

#undef LOCTEXT_NAMESPACE