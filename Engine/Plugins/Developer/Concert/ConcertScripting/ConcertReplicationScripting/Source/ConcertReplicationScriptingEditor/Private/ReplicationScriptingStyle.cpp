// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationScriptingStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Layout/Margin.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

namespace UE::ConcertReplicationScriptingEditor
{
	TSharedPtr<FSlateStyleSet> FReplicationScriptingStyle::StyleSet;

	FName FReplicationScriptingStyle::GetStyleSetName()
	{
		return FName(TEXT("ReplicationScriptingStyle"));
	}

	void FReplicationScriptingStyle::Initialize()
	{
		// Only register once
		if (StyleSet.IsValid())
		{
			return;
		}

		StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// This stuff is copied to look similar like the gameplay tag chips
		const FLinearColor ChipColor = FStyleColors::Hover.GetSpecifiedColor();
		const FLinearColor ChipColorHover = FStyleColors::Hover2.GetSpecifiedColor();
		const FLinearColor ChipColorDisable = ChipColor.CopyWithNewOpacity(0.35f);
		FButtonStyle ChipButtonSelected = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		ChipButtonSelected
			.SetNormal(FSlateRoundedBoxBrush(ChipColor, 3.0f))
			.SetHovered(FSlateRoundedBoxBrush(ChipColorHover, 3.0f))
			.SetPressed(FSlateRoundedBoxBrush(ChipColorHover, 3.0f))
			.SetDisabled(FSlateRoundedBoxBrush(ChipColorDisable, 3.0f))
			.SetNormalForeground(FStyleColors::ForegroundHover)
			.SetHoveredForeground(FStyleColors::White)
			.SetPressedForeground(FStyleColors::White)
			.SetDisabledForeground(FStyleColors::ForegroundHover)
			.SetNormalPadding(FMargin(5,2,2,2))
			.SetPressedPadding(FMargin(5,3,2,1));
		StyleSet->Set("ConcertProperty.ChipButton.Selected", ChipButtonSelected);

		FButtonStyle ChipClearButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		ChipClearButton.SetNormal(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetDisabled(FSlateNoResource())
			.SetNormalForeground(FStyleColors::ForegroundHeader)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Foreground)
			.SetNormalPadding(FMargin(2,0,2,0))
			.SetPressedPadding(FMargin(2,0,2,0));
		StyleSet->Set("ConcertProperty.ChipClearButton", ChipClearButton);
		
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
	};

	void FReplicationScriptingStyle::Shutdown()
	{
		if (StyleSet.IsValid())
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
			ensure(StyleSet.IsUnique());
			StyleSet.Reset();
		}
	}

	TSharedPtr<ISlateStyle> FReplicationScriptingStyle::Get()
	{
		return StyleSet;
	}
}


