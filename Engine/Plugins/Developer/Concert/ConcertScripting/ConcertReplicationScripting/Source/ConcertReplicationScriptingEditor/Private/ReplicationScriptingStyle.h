// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

namespace UE::ConcertReplicationScriptingEditor
{
	class FReplicationScriptingStyle
	{
	public:
	
		static void Initialize();
		static void Shutdown();

		static TSharedPtr<ISlateStyle> Get();
		static FName GetStyleSetName();
	
	private:
	
		static TSharedPtr<FSlateStyleSet> StyleSet;
	};
}


