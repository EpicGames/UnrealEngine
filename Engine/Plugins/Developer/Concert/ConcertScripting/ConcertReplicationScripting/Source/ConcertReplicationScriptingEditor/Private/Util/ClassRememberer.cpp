// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassRememberer.h"

#include "PropertyHandle.h"

namespace UE::ConcertReplicationScriptingEditor
{
	const UClass* FClassRememberer::GetLastUsedContextClassFor(IPropertyHandle& Property) const
	{
		const FString PropertyPath(Property.GetPropertyPath());
		const TWeakObjectPtr<const UClass>* LastBoundClass = PropertyToLastUsedClass.Find(PropertyPath);
		const TWeakObjectPtr<const UClass> ClassToUse = LastBoundClass ? *LastBoundClass : LastSelectedClass;
		return ClassToUse.IsValid() ? ClassToUse.Get() : nullptr;
	}
	
	void FClassRememberer::OnUseClass(IPropertyHandle& Property, const UClass* Class)
	{
		if (Class)
		{
			LastSelectedClass = Class;
			const FString PropertyPath(Property.GetPropertyPath());
			PropertyToLastUsedClass.Add(PropertyPath, Class);
		}
	}
}
