// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Templates/UnrealTemplate.h"

class IPropertyHandle;
class UClass;

namespace UE::ConcertReplicationScriptingEditor
{
	/** Remembers which classes were used based per customized property so users do not constantly have to re-select it in the drop-down menus. */
	class FClassRememberer : public FNoncopyable
	{
	public:

		/** Gets the class to use for searching properties in the FConcertPropertyChainWrapper or FConcertPropertyChainWrapperContainer. */
		const UClass* GetLastUsedContextClassFor(IPropertyHandle& Property) const;
		
		/** Caches a class for the property. */
		void OnUseClass(IPropertyHandle& Property, const UClass* Class);
	
	private:
		
		/** Keeps track of the last class that was used for searching properties in SConcertPropertyChainCombo. */
		TWeakObjectPtr<const UClass> LastSelectedClass;
		/**
		 * Maps every property path to the last used class. This why when a user works with multiple properties, each one will remember the last used class for convenience.
		 * This leaks memory but a user must edit a lot of properties to really be noticeable... should be ok.
		 */
		TMap<FString, TWeakObjectPtr<const UClass>> PropertyToLastUsedClass;
	};
}

