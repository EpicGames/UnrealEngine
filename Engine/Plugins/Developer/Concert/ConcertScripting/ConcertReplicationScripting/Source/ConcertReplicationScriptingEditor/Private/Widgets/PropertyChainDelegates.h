// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class UClass;
struct FConcertPropertyChain;

namespace UE::ConcertReplicationScriptingEditor
{
	DECLARE_DELEGATE(FOnPressed);
	
	DECLARE_DELEGATE_OneParam(FOnClassChanged, const UClass* Class);
	DECLARE_DELEGATE_TwoParams(FOnSelectedPropertiesChanged, const FConcertPropertyChain& Property, bool bIsSelected);
}