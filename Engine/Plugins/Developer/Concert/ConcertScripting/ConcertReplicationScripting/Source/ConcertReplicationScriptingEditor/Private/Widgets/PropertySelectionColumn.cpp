// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySelectionColumn.h"
#include "Replication/Editor/View/Column/ReplicationColumnsUtils.h"

#include "Algo/AnyOf.h"

#define LOCTEXT_NAMESPACE "PropertySelectionCheckboxColumn"

namespace UE::ConcertReplicationScriptingEditor
{
	const FName PropertySelectionCheckboxColumnId(TEXT("AddPropertyCheckboxColumn"));
	
	ConcertSharedSlate::FPropertyColumnEntry MakePropertySelectionCheckboxColumn(
		const TSet<FConcertPropertyChain>& SelectedProperties,
		FOnSelectProperty OnSelectPropertyDelegate,
		bool bIsEditable,
		int32 SortPriority
		)
	{
		using namespace ConcertSharedSlate;
		using FPropertyColumnDelegates = TCheckboxColumnDelegates<FPropertyTreeRowContext>;
		return MakeCheckboxColumn<FPropertyTreeRowContext>(
			PropertySelectionCheckboxColumnId,
			FPropertyColumnDelegates(
				FPropertyColumnDelegates::FGetColumnCheckboxState::CreateLambda(
				[&SelectedProperties](const FPropertyTreeRowContext& Data)
				{
					const FConcertPropertyChain& DisplayProperty = Data.RowData.GetProperty();
					const bool bShouldBeSelected = SelectedProperties.Contains(DisplayProperty)
						|| Algo::AnyOf(SelectedProperties, [&DisplayProperty](const FConcertPropertyChain& SelectedProperty)
						{
							return SelectedProperty.IsChildOf(DisplayProperty);
						});
					return bShouldBeSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}),
				FPropertyColumnDelegates::FOnColumnCheckboxChanged::CreateLambda(
				[OnSelectPropertyDelegate = MoveTemp(OnSelectPropertyDelegate)](bool bIsChecked, const FPropertyTreeRowContext& Data)
				{
					OnSelectPropertyDelegate.Execute(Data.RowData.GetProperty(), bIsChecked);
				}),
				FPropertyColumnDelegates::FGetToolTipText::CreateLambda(
				[](const FPropertyTreeRowContext&)
				{
					return LOCTEXT("IncludePropertyTooltip", "Whether the property is included");
				}),
				FPropertyColumnDelegates::FIsEnabled::CreateLambda([bIsEditable](const FPropertyTreeRowContext&)
				{
					return bIsEditable;
				})),
			FText::GetEmpty(),
			SortPriority,
			20.f
		);
	}
}

#undef LOCTEXT_NAMESPACE