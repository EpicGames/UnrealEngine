// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/Column/IPropertyTreeColumn.h"
#include "Delegates/Delegate.h"

namespace UE::ConcertReplicationScriptingEditor
{
	extern const FName PropertySelectionCheckboxColumnId;
	
	DECLARE_DELEGATE_TwoParams(FOnSelectProperty, const FConcertPropertyChain& PropertyChain, bool bIsSelected);

	/**
	 * Creates a checkbox column.
	 *
	 * @param SelectedProperties The checkbox is checked if the path is in SelectedProperties.
	 * @param OnSelectPropertyDelegate Executed when a checkbox is checked or unchecked.
	 */
	ConcertSharedSlate::FPropertyColumnEntry MakePropertySelectionCheckboxColumn(
		const TSet<FConcertPropertyChain>& SelectedProperties,
		FOnSelectProperty OnSelectPropertyDelegate,
		bool bIsEditable,
		int32 SortPriority = 0 /* This places it before label column by default */
		);
}

