// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertPropertyChainChip.h"

#include "ReplicationScriptingStyle.h"
#include "Replication/Data/ConcertPropertySelection.h"

#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SConcertPropertyChainChip"

namespace UE::ConcertReplicationScriptingEditor
{
	void SConcertPropertyChainChip::Construct(const FArguments& InArgs)
	{
		const FConcertPropertyChain PropertyChain = InArgs._DisplayedProperty.Get();
		ChildSlot
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(PropertyChain.IsEmpty() ? 1 : 0)

			+SWidgetSwitcher::Slot()
			[
				SNew(SHorizontalBox)
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FReplicationScriptingStyle::Get(), "ConcertProperty.ChipButton.Selected")
					.OnClicked_Lambda([OnEditPressed = InArgs._OnEditPressed](){ OnEditPressed.ExecuteIfBound(); return FReply::Handled(); })
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
							.Text(FText::FromString(PropertyChain.ToString()))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.Visibility(!PropertyChain.IsEmpty() && InArgs._ShowClearButton ? EVisibility::Visible : EVisibility::Collapsed)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.ButtonStyle(FReplicationScriptingStyle::Get(), "ConcertProperty.ChipClearButton")
							.ToolTipText(LOCTEXT("Remove", "Remove"))
							.ContentPadding(FMargin(2, 0, 0, 0))
							.OnClicked_Lambda([OnClearPressed = InArgs._OnClearPressed]()
							{
								OnClearPressed.ExecuteIfBound();
								return FReply::Handled();
							})
							[
								SNew(SImage)
								.ColorAndOpacity(FStyleColors::Foreground)
								.Image(FAppStyle::GetBrush("Icons.X"))
								.DesiredSizeOverride(FVector2D(12.0f, 12.0f))
							]
						]
					]
				]
				
				// So that the chip does not take up the entire row and leaves a lot of empty space next to the remove button
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				]
			]
			
			+SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("EmptyPropertyChain.Label", "Empty"))
				.ToolTipText(LOCTEXT("EmptyPropertyChain.Tooltip", "Empty property"))
			]
			
		];
	}
}

#undef LOCTEXT_NAMESPACE