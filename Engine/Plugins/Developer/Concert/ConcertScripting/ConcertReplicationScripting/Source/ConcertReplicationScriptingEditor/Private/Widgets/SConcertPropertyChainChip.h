// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyChainDelegates.h"
#include "Replication/Data/ConcertPropertySelection.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertReplicationScriptingEditor
{
	/** Displays a FConcertPropertyChain */
	class SConcertPropertyChainChip : public SCompoundWidget
	{
	public:
		
		SLATE_BEGIN_ARGS(SConcertPropertyChainChip)
		{}
			/** The property to display */
			SLATE_ATTRIBUTE(FConcertPropertyChain, DisplayedProperty)

			/** Whether the X-Button should be shown */
			SLATE_ARGUMENT(bool, ShowClearButton)

			/** Called when the X-button is pressed */
			SLATE_EVENT(FOnPressed, OnClearPressed)

			/** Called when the chip is pressed. */
			SLATE_EVENT(FOnPressed, OnEditPressed)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
	};
}
