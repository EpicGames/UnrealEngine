// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertPropertyChainCombo.h"

#include "ConcertPropertyChainWrapper.h"
#include "SConcertPropertyChainChip.h"
#include "SConcertPropertyChainPicker.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SConcertPropertyChainCombo"

namespace UE::ConcertReplicationScriptingEditor
{
	void SConcertPropertyChainCombo::Construct(const FArguments& InArgs)
	{
		LastClassUsed = InArgs._InitialClassSelection;
		ContainedProperties = InArgs._ContainedProperties;
		bIsEditable = InArgs._IsEditable;
		HasMultipleValuesAttribute = InArgs._HasMultipleValues;
		OnClassChangedDelegate = InArgs._OnClassChanged;
		OnPropertySelectionChangedDelegate = InArgs._OnPropertySelectionChanged;
		
		ChildSlot
		[
			SAssignNew(ComboButton, SComboButton)
			.HasDownArrow(true)
			.ContentPadding(1)
			.OnGetMenuContent(this, &SConcertPropertyChainCombo::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([this]()
				{
					return HasMultipleValuesAttribute.Get()
						? 0
						: DisplayedProperties.IsEmpty()
							? 1
							: 2;
				})
				
				+SWidgetSwitcher::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MultipleValues", "Multiple Values"))
				]
				+SWidgetSwitcher::Slot()
				[
					SNew(SConcertPropertyChainChip)
					.DisplayedProperty(FConcertPropertyChain{})
					.ShowClearButton(false)
				]
				+SWidgetSwitcher::Slot()
				[
					SAssignNew(PropertyListView, SListView<TSharedPtr<FConcertPropertyChain>>)
					.ListItemsSource(&DisplayedProperties)
					.SelectionMode(ESelectionMode::None)
					.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
					.OnGenerateRow(this, &SConcertPropertyChainCombo::MakePropertyChainRow)
				]
			]
		];

		RefreshPropertyContent();
	}

	void SConcertPropertyChainCombo::RefreshPropertyContent()
	{
		DisplayedProperties.Reset();
		ON_SCOPE_EXIT{ PropertyListView->RequestListRefresh(); };
		
		if (HasMultipleValuesAttribute.Get() || !ensureAlwaysMsgf(ContainedProperties, TEXT("You forgot to specified ContainedProperties during construction!")))
		{
			return;
		}

		for (const FConcertPropertyChain& Property : *ContainedProperties)
		{
			DisplayedProperties.Emplace(MakeShared<FConcertPropertyChain>(Property));
		}
		DisplayedProperties.Sort([](const TSharedPtr<FConcertPropertyChain>& Left, const TSharedPtr<FConcertPropertyChain>& Right)
		{
			return Left->ToString() <= Right->ToString();
		});
	}

	TSharedRef<SWidget> SConcertPropertyChainCombo::OnGetMenuContent()
	{
		 const TSharedRef<SConcertPropertyChainPicker> Menu = SNew(SConcertPropertyChainPicker)
			.ContainedProperties(ContainedProperties)
			.IsEditable(bIsEditable)
			.InitialClassSelection(LastClassUsed)
			.OnClassChanged_Lambda([this](const UClass* Class)
			{
				LastClassUsed = Class;
				OnClassChangedDelegate.ExecuteIfBound(Class);
			})
			.OnSelectedPropertiesChanged(OnPropertySelectionChangedDelegate);
		WeakButtonMenu = Menu;
		return Menu;
	}

	TSharedRef<ITableRow> SConcertPropertyChainCombo::MakePropertyChainRow(TSharedPtr<FConcertPropertyChain> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		const FConcertPropertyChain PropertyChain = *Item.Get();
		return SNew(STableRow<TSharedPtr<FConcertPropertyChain>>, OwnerTable)
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row"))
			.Padding(FMargin(0,2))
			[
				SNew(SConcertPropertyChainChip)
				.DisplayedProperty(*Item)
				.ShowClearButton(bIsEditable)
				.OnClearPressed_Lambda([this, PropertyChain](){ OnPropertySelectionChangedDelegate.ExecuteIfBound(PropertyChain, false); })
				.OnEditPressed_Lambda([this, PropertyChain]()
				{
					// Handle being pressed while menu is open already
					if (!ComboButton->IsOpen())
					{
						ComboButton->SetIsOpen(true);
					}
					
					// Should have been created by SetIsOpen 
					if (const TSharedPtr<SConcertPropertyChainPicker> MenuPin = WeakButtonMenu.Pin())
					{
						MenuPin->RequestScrollIntoView(PropertyChain);
					}
				})
			];
	}
}

#undef LOCTEXT_NAMESPACE