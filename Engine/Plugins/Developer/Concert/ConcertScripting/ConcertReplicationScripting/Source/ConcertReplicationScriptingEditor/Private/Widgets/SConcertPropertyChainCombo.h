// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"
#include "PropertyChainDelegates.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SComboButton;

namespace UE::ConcertReplicationScriptingEditor
{
	class SConcertPropertyChainPicker;
	/**
	 * A combo button that displays a list of properties as content and as menu opens a property tree view for editing
	 * the list of properties.
	 */
	class SConcertPropertyChainCombo : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SConcertPropertyChainCombo)
			: _InitialClassSelection(nullptr)
		{}
			/**
			 * The properties that are supposed to be displayed inside of the button.
			 * 
			 * This must always contain at least one property. If there are "no" properties, this should contain the empty, default constructed FConcertPropertyChain.
			 * This requirement is needed so the button shows the "Empty" widget.
			 */
			SLATE_ARGUMENT(const TSet<FConcertPropertyChain>*, ContainedProperties)

			/** Whether the UI should allow editing */
			SLATE_ARGUMENT(bool, IsEditable)
			/** If this returns true, then the button will display "Multiple Values" instead. */
			SLATE_ATTRIBUTE(bool, HasMultipleValues)
		
			/** The initial class to search in */
			SLATE_ARGUMENT(const UClass*, InitialClassSelection)
		
			/** Called when the class is changed */
			SLATE_EVENT(FOnClassChanged, OnClassChanged)
			/** Called when the selection of a property changes. */
			SLATE_EVENT(FOnSelectedPropertiesChanged, OnPropertySelectionChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Regenerates the button's content. */
		void RefreshPropertyContent();
		
	private:

		/** Class set by the last menu this widget had used. */
		const UClass* LastClassUsed = nullptr;
		/** The properties that are supposed to be displayed inside of the button */
		const TSet<FConcertPropertyChain>* ContainedProperties;

		/** Whether the UI should allow editing. */
		bool bIsEditable;

		/** The combo button being displayed. */
		TSharedPtr<SComboButton> ComboButton;
		/** Used to scroll into view when clicking an item in the button content. */
		TWeakPtr<SConcertPropertyChainPicker> WeakButtonMenu;
		
		/** The properties displayed in PropertyListView. */
		TArray<TSharedPtr<FConcertPropertyChain>> DisplayedProperties;
		/** The content of this button */
		TSharedPtr<SListView<TSharedPtr<FConcertPropertyChain>>> PropertyListView;

		/** If true, then ContainedProperties contains the accumulated values of multiple sources. If false, all sources have the same value. */
		TAttribute<bool> HasMultipleValuesAttribute;
		
		/** Triggered when the class changes */
		FOnClassChanged OnClassChangedDelegate;
		/** Called when the selection of a property changes. */
		FOnSelectedPropertiesChanged OnPropertySelectionChangedDelegate;

		/** Generates the combo button menu content. */
		TSharedRef<SWidget> OnGetMenuContent();
		
		TSharedRef<ITableRow> MakePropertyChainRow(TSharedPtr<FConcertPropertyChain> Item, const TSharedRef<STableViewBase>& OwnerTable);
	};
}

