// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConcertPropertyChainPicker.h"

#include "PropertyCustomizationHelpers.h"
#include "PropertySelectionColumn.h"
#include "Replication/PropertyChainUtils.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Editor/View/IPropertyTreeView.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SConcertPropertyChainPicker"

namespace UE::ConcertReplicationScriptingEditor
{
	void SConcertPropertyChainPicker::Construct(const FArguments& InArgs)
	{
		SelectedClass = InArgs._InitialClassSelection;
		ContainedProperties = InArgs._ContainedProperties;
		OnClassChangedDelegate = InArgs._OnClassChanged;
		OnSelectedPropertiesChangedDelegate = InArgs._OnSelectedPropertiesChanged;
		
		ChildSlot
		[
			SNew(SBox)
			.MinDesiredHeight(750.f)
			.MaxDesiredHeight(750.f)
			.MinDesiredWidth(350.f)
			.MaxDesiredWidth(350.f)
			[
				SNew(SBorder)
				.Padding(2.f)
				.BorderImage(FStyleDefaults::GetNoBrush())
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.f)
					[
						CreateClassPickerSection()
					]

					+SVerticalBox::Slot()
					.FillHeight(1.f)
					.Padding(2.f)
					[
						CreatePropertyTreeView(InArgs)
					]
				]
			]
		];
		
		RefreshPropertiesDisplayedInTree();
	}

	void SConcertPropertyChainPicker::RequestScrollIntoView(const FConcertPropertyChain& PropertyChain)
	{
		TreeView->RequestScrollIntoView(PropertyChain);
	}

	TSharedRef<SWidget> SConcertPropertyChainPicker::CreateClassPickerSection()
	{
		return SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("Class.Tooltip", "The class for which to display the tree hierarchy"))
				
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Class.Label", "Class Context"))
			]
				
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(50.f, 0.f, 0.f, 0.f)
			[
				SNew(SClassPropertyEntryBox)
				.MetaClass(UObject::StaticClass())
				.SelectedClass_Lambda([this]() { return SelectedClass; })
				.OnSetClass(this, &SConcertPropertyChainPicker::OnSetClass)
			];
	}

	TSharedRef<SWidget> SConcertPropertyChainPicker::CreatePropertyTreeView(const FArguments& InArgs)
	{
		ConcertSharedSlate::FCreatePropertyTreeViewParams Params
		{
			.PropertyColumns ={ ConcertSharedSlate::ReplicationColumns::Property::LabelColumn()}
		};
		Params.NoItemsContent.Widget = SNew(STextBlock).Text(LOCTEXT("NoClass", "Select a class"));

		if (ensureAlwaysMsgf(ContainedProperties, TEXT("You forgot to pass in the required ContainedProperties argument for widget construction!")))
		{
			Params.PrimaryPropertySort = { PropertySelectionCheckboxColumnId, EColumnSortMode::Ascending };
			Params.PropertyColumns.Add(
				MakePropertySelectionCheckboxColumn(
					*ContainedProperties,
					FOnSelectProperty::CreateSP(this, &SConcertPropertyChainPicker::OnPropertySelected),
					InArgs._IsEditable
					)
				);
		}
		
		TreeView = ConcertSharedSlate::CreateSearchablePropertyTreeView(Params);
		return TreeView->GetWidget();
	}
	
	void SConcertPropertyChainPicker::OnSetClass(const UClass* Class)
	{
		SelectedClass = Class;
		OnClassChangedDelegate.ExecuteIfBound(Class);
		RefreshPropertiesDisplayedInTree();
	}

	void SConcertPropertyChainPicker::RefreshPropertiesDisplayedInTree()
	{
		ConcertSharedSlate::FPropertyAssignmentEntry Entry { .Class = SelectedClass };
		TSet<FConcertPropertyChain>& Properties = Entry.PropertiesToDisplay;
		
		if (SelectedClass)
		{
			ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*SelectedClass, [&Properties](FConcertPropertyChain&& PropertyChain)
			{
				Properties.Emplace(MoveTemp(PropertyChain));
				return EBreakBehavior::Continue;
			});
		}

		// If no class is selected, the widget will display the NoItemsContent ("Select a class").
		TreeView->RefreshPropertyData({ Entry });
	}

	void SConcertPropertyChainPicker::OnPropertySelected(const FConcertPropertyChain& ConcertPropertyChain, bool bIsSelected)
	{
		OnSelectedPropertiesChangedDelegate.ExecuteIfBound(ConcertPropertyChain, bIsSelected);

		// The checkbox state may have changed so request a resort.
		TreeView->RequestResortForColumn(PropertySelectionCheckboxColumnId);
	}
}

#undef LOCTEXT_NAMESPACE