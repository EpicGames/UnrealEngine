// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"
#include "PropertyChainDelegates.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	class IPropertyTreeView;
}

namespace UE::ConcertReplicationScriptingEditor
{
	
	/** Menu that has a class picker and tree view of properties in the class. */
	class SConcertPropertyChainPicker : public SCompoundWidget
	{
	public:
		
		SLATE_BEGIN_ARGS(SConcertPropertyChainPicker)
			: _InitialClassSelection(nullptr)
			, _ContainedProperties(nullptr)
		{}
			/** The initial class to search in */
			SLATE_ARGUMENT(const UClass*, InitialClassSelection)
			/** Whether the UI should allow editing */
			SLATE_ARGUMENT(bool, IsEditable)
		
			/** The properties that are supposed to be displayed inside of the button */
			SLATE_ARGUMENT(const TSet<FConcertPropertyChain>*, ContainedProperties)
		
			/** Called when the class is changed */
			SLATE_EVENT(FOnClassChanged, OnClassChanged)
			/** Called when the user's property selection changes */
			SLATE_EVENT(FOnSelectedPropertiesChanged, OnSelectedPropertiesChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		void RequestScrollIntoView(const FConcertPropertyChain& PropertyChain);
			
	private:
		
		/** The class for which to display the hierarchy*/
		const UClass* SelectedClass = nullptr;
		/** The properties that are supposed to be displayed inside of the button */
		const TSet<FConcertPropertyChain>* ContainedProperties;

		/** Displays the properties */
		TSharedPtr<ConcertSharedSlate::IPropertyTreeView> TreeView;
		
		/** Triggered when the class changes */
		FOnClassChanged OnClassChangedDelegate;
		/** Called when the user's property selection changes */
		FOnSelectedPropertiesChanged OnSelectedPropertiesChangedDelegate;

		TSharedRef<SWidget> CreateClassPickerSection();
		TSharedRef<SWidget> CreatePropertyTreeView(const FArguments& InArgs);
		
		void OnSetClass(const UClass* Class);
		void RefreshPropertiesDisplayedInTree();
		void OnPropertySelected(const FConcertPropertyChain& ConcertPropertyChain, bool bIsSelected);
	};
}

